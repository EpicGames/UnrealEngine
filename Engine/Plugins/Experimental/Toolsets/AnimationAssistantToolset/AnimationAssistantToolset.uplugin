{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "AnimationAssistantToolset",
	"Description": "Toolset for Animation Systems",
	"Category": "Other",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EditorOnly": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimationAssistantToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		}
	]
}